// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraMediaMJPEGDecoder.h"
#include "MJPEGDecoderElectraModule.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "Utils/ElectraBitstreamProcessorDefault.h"

#include "ElectraDecodersPlatformResources.h"

#include "ImageUtils.h"

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_ALREADY_CLOSED						1
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				2

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FVideoDecoderMJPEGElectra;


class FVideoDecoderOutputMJPEGElectra : public IElectraDecoderVideoOutput, public IElectraDecoderVideoOutputImageBuffers
{
public:
	virtual ~FVideoDecoderOutputMJPEGElectra()
	{
	}

	FTimespan GetPTS() const override
	{
		return PTS;
	}
	uint64 GetUserValue() const override
	{
		return UserValue;
	}

	EOutputType GetOutputType() const
	{
		return EOutputType::Output;
	}

	int32 GetWidth() const override
	{
		return Width - Crop.Left - Crop.Right;
	}
	int32 GetHeight() const override
	{
		return Height - Crop.Top - Crop.Bottom;
	}
	int32 GetDecodedWidth() const override
	{
		return Width;
	}
	int32 GetDecodedHeight() const override
	{
		return Height;
	}
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{
		return Crop;
	}
	int32 GetAspectRatioW() const override
	{
		return AspectW;
	}
	int32 GetAspectRatioH() const override
	{
		return AspectH;
	}
	int32 GetFrameRateNumerator() const override
	{
		return FrameRateN;
	}
	int32 GetFrameRateDenominator() const override
	{
		return FrameRateD;
	}
	int32 GetNumberOfBits() const override
	{
		return NumBits;
	}
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{
		OutExtraValues = ExtraValues;
	}
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::ImageBuffers)
		{
			return static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FVideoDecoderOutputMJPEGElectra*>(this));
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{
		return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported;
	}

	// Methods from IElectraDecoderVideoOutputImageBuffers
	uint32 GetCodec4CC() const override
	{
		return Codec4CC;
	}
	int32 GetNumberOfBuffers() const override
	{
		return NumBuffers;
	}
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffers[EBufferType::Buffer_Color].Buffer;
		}
		else if (InBufferIndex == 1)
		{
			return Buffers[EBufferType::Buffer_Alpha].Buffer;
		}
		return nullptr;
	}
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
		return nullptr;
	}
	bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
		return false;
	}

	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffers[EBufferType::Buffer_Color].BufferFormat;
		}
		else if (InBufferIndex == 1)
		{
			return Buffers[EBufferType::Buffer_Alpha].BufferFormat;
		}
		return EPixelFormat::PF_Unknown;
	}
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffers[EBufferType::Buffer_Color].BufferEncoding;
		}
		else if (InBufferIndex == 1)
		{
			return Buffers[EBufferType::Buffer_Alpha].BufferEncoding;
		}
		return EElectraTextureSamplePixelEncoding::Native;
	}
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffers[EBufferType::Buffer_Color].Pitch;
		}
		else if (InBufferIndex == 1)
		{
			return Buffers[EBufferType::Buffer_Alpha].Pitch;
		}
		return 0;
	}

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 Width = 0;
	int32 Height = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	TMap<FString, FVariant> ExtraValues;

	uint32 Codec4CC = 0;
	int32 NumBuffers = 0;

	enum EBufferType
	{
		Buffer_Color = 0,
		Buffer_Alpha = 1
	};

	struct FBufferInfo
	{
		TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> Buffer;
		EPixelFormat BufferFormat;
		EElectraTextureSamplePixelEncoding BufferEncoding;
		int32 Pitch = 0;
	};
	FBufferInfo Buffers[2];
};


class FVideoDecoderMJPEGElectra : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
		OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)5));
		OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(true));
	}

	FVideoDecoderMJPEGElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FVideoDecoderMJPEGElectra();

	IElectraDecoder::EType GetType() const override
	{
		return IElectraDecoder::EType::Video;
	}

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{ return FElectraDecoderBitstreamProcessorDefault::Create(); }

	void Suspend() override
	{ }
	void Resume() override
	{ }

private:
	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);


	int32 DisplayWidth = 0;
	int32 DisplayHeight = 0;
	int32 AspectW = 0;
	int32 AspectH = 0;

	IElectraDecoder::FError LastError;
	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;

	TSharedPtr<FVideoDecoderOutputMJPEGElectra, ESPMode::ThreadSafe> CurrentOutput;
	bool bFlushPending = false;
};


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FMJPEGVideoDecoderElectraFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FMJPEGVideoDecoderElectraFactory, ESPMode::ThreadSafe>
{
public:
	virtual ~FMJPEGVideoDecoderElectraFactory()
	{ }

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FVideoDecoderMJPEGElectra::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		switch(InCodecFormat.FourCC)
		{
			case ElectraDecodersUtil::Make4CC('j','p','e','g'):
			{
				if (InCodecFormat.HumanReadableFormatInfo.IsEmpty())
				{
					OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, FVariant(FString(TEXT("MotionJPEG"))));
				}
				else
				{
					OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, InCodecFormat.HumanReadableFormatInfo);
				}
				OutFormatInfo.Emplace(IElectraDecoderFormatInfo::IsEveryFrameKeyframe, FVariant(true));
				return 1;
			}
			default:
			{
				break;
			}
		}
		return 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? MakeShared<FVideoDecoderMJPEGElectra, ESPMode::ThreadSafe>(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("ImageUtils")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString()); }

	static TSharedPtr<FMJPEGVideoDecoderElectraFactory, ESPMode::ThreadSafe> Self;
};
TSharedPtr<FMJPEGVideoDecoderElectraFactory, ESPMode::ThreadSafe> FMJPEGVideoDecoderElectraFactory::Self;

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

void FElectraMediaMJPEGDecoder::Startup()
{
	// Make sure the codec factory module has been loaded.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

	// Create an instance of the factory, which is also the modular feature.
	check(!FMJPEGVideoDecoderElectraFactory::Self.IsValid());
	FMJPEGVideoDecoderElectraFactory::Self = MakeShared<FMJPEGVideoDecoderElectraFactory, ESPMode::ThreadSafe>();
	// Register as modular feature.
	IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FMJPEGVideoDecoderElectraFactory::Self.Get());
}

void FElectraMediaMJPEGDecoder::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FMJPEGVideoDecoderElectraFactory::Self.Get());
	FMJPEGVideoDecoderElectraFactory::Self.Reset();
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

FVideoDecoderMJPEGElectra::FVideoDecoderMJPEGElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	const Electra::FCodecTypeFormat::FVideo& vid(InitialCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
	DisplayWidth = (int32) vid.Width;
	DisplayHeight = (int32) vid.Height;
	AspectW = (int32) vid.AspectRatioW;
	AspectH = (int32) vid.AspectRatioH;
}

FVideoDecoderMJPEGElectra::~FVideoDecoderMJPEGElectra()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

void FVideoDecoderMJPEGElectra::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FVideoDecoderMJPEGElectra::GetError() const
{
	return LastError;
}

bool FVideoDecoderMJPEGElectra::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

void FVideoDecoderMJPEGElectra::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FVideoDecoderMJPEGElectra::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	return IElectraDecoder::ECSDCompatibility::Compatible;
}

bool FVideoDecoderMJPEGElectra::ResetToCleanStart()
{
	bFlushPending = false;
	CurrentOutput.Reset();
	return true;
}


IElectraDecoder::EDecoderError FVideoDecoderMJPEGElectra::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// Decode data. This immediately produces a new output frame.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		FImage Image;
		if (!FImageUtils::DecompressImage(InInputAccessUnit.Data, InInputAccessUnit.DataSize, Image))
		{
			PostError(0, TEXT("JPEG decoding failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		if (Image.Format != ERawImageFormat::BGRA8)
		{
			PostError(0, TEXT("Decoded JPEG image is not BGRA"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		int32 DecodedWidth = Image.SizeX;
		int32 DecodedHeight = Image.SizeY;

		TSharedPtr<FVideoDecoderOutputMJPEGElectra, ESPMode::ThreadSafe> NewOutput = MakeShared<FVideoDecoderOutputMJPEGElectra>();
		NewOutput->PTS = InInputAccessUnit.PTS;
		NewOutput->UserValue = InInputAccessUnit.UserValue;

		NewOutput->Width = DecodedWidth;
		NewOutput->Height = DecodedHeight;
		NewOutput->Crop.Right = DecodedWidth > DisplayWidth ? DecodedWidth - DisplayWidth : 0;
		NewOutput->Crop.Bottom = DecodedHeight > DisplayHeight ? DecodedHeight - DisplayHeight : 0;
		if (AspectW && AspectH)
		{
			NewOutput->AspectW = AspectW;
			NewOutput->AspectH = AspectH;
		}

		NewOutput->NumBits = 8;

		NewOutput->NumBuffers = 1;
		TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> Buf = MakeShareable(new TArray64<uint8>());
		*Buf = MoveTemp(Image.RawData);
		NewOutput->Buffers[0].Buffer = MoveTemp(Buf);
		NewOutput->Buffers[0].BufferFormat = EPixelFormat::PF_B8G8R8A8;
		NewOutput->Buffers[0].BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
		NewOutput->Buffers[0].Pitch = DecodedWidth * 4;
		NewOutput->Codec4CC = InitialCodecFormat.FourCC;
		NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("mjpg")));
		NewOutput->ExtraValues.Emplace(TEXT("codec_4cc"), FVariant(InitialCodecFormat.FourCC));

		CurrentOutput = MoveTemp(NewOutput);
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderMJPEGElectra::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Already draining?
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	bFlushPending = true;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderMJPEGElectra::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	ResetToCleanStart();
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FVideoDecoderMJPEGElectra::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}
	// Have output?
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}
	// Pending flush?
	if (bFlushPending)
	{
		bFlushPending = false;
		return IElectraDecoder::EOutputStatus::EndOfData;
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FVideoDecoderMJPEGElectra::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}
