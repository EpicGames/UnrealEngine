// Copyright Epic Games, Inc. All Rights Reserved.

#include "MJPEGDecoderElectraModule.h"
#include "MJPEGDecoder/ElectraMediaMJPEGDecoder.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "MJPEGDecoderElectraModule"

DEFINE_LOG_CATEGORY(LogMJPEGElectraDecoder);

class FMJPEGElectraDecoderModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
		FElectraMediaMJPEGDecoder::Startup();
	}

	void ShutdownModule() override
	{
		FElectraMediaMJPEGDecoder::Shutdown();
	}

	bool SupportsDynamicReloading() override
	{
		// Codec could still be in use
		return false;
	}
};

IMPLEMENT_MODULE(FMJPEGElectraDecoderModule, MJPEGDecoderElectra);

#undef LOCTEXT_NAMESPACE
